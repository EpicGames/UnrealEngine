{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "CodeLite Integration",
	"Description" : "Allows access to source code in CodeLite.",
	"Category" : "Programming",
	"CreatedBy" : "Cengiz Terzibas (yaakuro)",
	"CreatedByURL" : "",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "CodeLiteSourceCodeAccess",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Linux"
			]
		}
	]
}