// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CodeLiteSourceCodeAccess : ModuleRules
	{
		public CodeLiteSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"SourceCodeAccess",
					"DesktopPlatform",
					"Projects",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("HotReload");
			}

			ShortName = "CLiteSCA";
		}
	}
}
