// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeLiteSourceCodeAccessModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"

IMPLEMENT_MODULE( FCodeLiteSourceCodeAccessModule, CodeLiteSourceCodeAccess );

#define LOCTEXT_NAMESPACE "CodeLiteSourceCodeAccessor"

void FCodeLiteSourceCodeAccessModule::StartupModule()
{
	CodeLiteSourceCodeAccessor.Startup();

	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &CodeLiteSourceCodeAccessor );
}

void FCodeLiteSourceCodeAccessModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &CodeLiteSourceCodeAccessor);

	CodeLiteSourceCodeAccessor.Shutdown();
}

FCodeLiteSourceCodeAccessor& FCodeLiteSourceCodeAccessModule::GetAccessor()
{
	return CodeLiteSourceCodeAccessor;
}

#undef LOCTEXT_NAMESPACE