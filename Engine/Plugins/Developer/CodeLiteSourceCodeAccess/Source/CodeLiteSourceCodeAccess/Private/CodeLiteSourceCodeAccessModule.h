// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "CodeLiteSourceCodeAccessor.h"

class FCodeLiteSourceCodeAccessModule : public IModuleInterface
{
public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

	FCodeLiteSourceCodeAccessor& GetAccessor();

private:

	FCodeLiteSourceCodeAccessor CodeLiteSourceCodeAccessor;
};
