// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeLiteSourceCodeAccessor.h"
#include "Misc/Paths.h"
#include "Misc/UProjectInfo.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Trace/Trace.inl"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogCodeLiteAccessor, Log, All);

#define LOCTEXT_NAMESPACE "CodeLiteSourceCodeAccessor"

static FString MakePath(const FString& InPath)
{
	return TEXT("\"") + InPath.ReplaceQuotesWithEscapedQuotes() + TEXT("\""); 
}

void FCodeLiteSourceCodeAccessor::Startup()
{
	// Cache this so we don't have to do it on a background thread
	GetSolutionPath();
}

void FCodeLiteSourceCodeAccessor::RefreshAvailability()
{
	FString OutURL;
	int32 ReturnCode = -1;

	FPlatformProcess::ExecProcess(TEXT("/usr/bin/env"), TEXT("bash -c \"type -p codelite\""), &ReturnCode, &OutURL, nullptr);
	if (ReturnCode == 0)
	{
		Location.URL = OutURL.TrimStartAndEnd();
	}
	else
	{
		// Fallback to default install location
		FString URL = TEXT("/usr/bin/codelite");
		if (FPaths::FileExists(URL))
		{
			Location.URL = URL;
		}
	}
}

void FCodeLiteSourceCodeAccessor::Shutdown()
{

}

bool FCodeLiteSourceCodeAccessor::CanAccessSourceCode() const
{
	return Location.IsValid();
}

FName FCodeLiteSourceCodeAccessor::GetFName() const
{
	return FName("CodeLiteSourceCodeAccessor");
}

FText FCodeLiteSourceCodeAccessor::GetNameText() const
{
	return LOCTEXT("CodeLiteDisplayName", "CodeLite 10/11/12");
}

FText FCodeLiteSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("CodeLiteDisplayDesc", "Open source code files in CodeLite");
}

bool FCodeLiteSourceCodeAccessor::OpenSolution()
{
	if (!Location.IsValid())
	{
		return false;
	}

	return OpenSolutionAtPath(GetSolutionPath());
}

bool FCodeLiteSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	if (!Location.IsValid())
	{
		return false;
	}

	FString SolutionPath = InSolutionPath;

	if (!SolutionPath.EndsWith(TEXT(".workspace")))
	{
		SolutionPath = SolutionPath + TEXT(".workspace");
	}

	TArray<FString> Args;
	Args.Add(MakePath(SolutionPath));
	return Launch(Args);
}

bool FCodeLiteSourceCodeAccessor::DoesSolutionExist() const
 {
	return FPaths::FileExists(GetSolutionPath());
 }

bool FCodeLiteSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	if (!Location.IsValid())
	{
		return false;
	}

	FString SolutionDir = GetSolutionPath();
	TArray<FString> Args;
	Args.Add(MakePath(SolutionDir));

	for (const FString& SourcePath : AbsoluteSourcePaths)
	{
		Args.Add(MakePath(SourcePath));
	}

	return Launch(Args);
}

bool FCodeLiteSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	if (!Location.IsValid())
	{
		return false;
	}

	LineNumber = LineNumber > 0 ? LineNumber : 1;

	FString SolutionDir = GetSolutionPath();
	TArray<FString> Args;
	Args.Add(MakePath(SolutionDir));
	Args.Add(MakePath(FullPath) + FString::Printf(TEXT(" --line=%d"), LineNumber));

	return Launch(Args);
}

bool FCodeLiteSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	// TODO Is this possible without dbus here? Maybe we can add this functionality to CodeLite.
	return false;
}

bool FCodeLiteSourceCodeAccessor::SaveAllOpenDocuments() const
{
	// TODO Is this possible without dbus here? Maybe we can add this functionality to CodeLite.
	return false;
}

void FCodeLiteSourceCodeAccessor::Tick(const float DeltaTime)
{
	// TODO What can we do here?
}

FString FCodeLiteSourceCodeAccessor::GetSolutionPath() const
{
	FScopeLock Lock(&CachedSolutionPathCriticalSection);

	if (IsInGameThread())
	{
		CachedSolutionPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

		FString PrimaryProjectPath;
		if (!FFileHelper::LoadFileToString(PrimaryProjectPath, *(FPaths::EngineIntermediateDir() / TEXT("ProjectFiles") / TEXT("PrimaryProjectPath.txt"))) || 
			(PrimaryProjectPath.TrimStartAndEndInline(), PrimaryProjectPath.IsEmpty()))
		{
			const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();
			const FString ProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

			if ((CurrentProject == nullptr || CurrentProject->Modules.Num() == 0) || !FUProjectDictionary::GetDefault().IsForeignProject(ProjectDir))
			{
				PrimaryProjectPath = FPaths::RootDir() / TEXT("UE5");
			}
			else
			{
				const FString BaseName = FApp::HasProjectName() ? FApp::GetProjectName() : FPaths::GetBaseFilename(ProjectDir);
				PrimaryProjectPath = ProjectDir / BaseName;
			}
		}

		CachedSolutionPath = PrimaryProjectPath + TEXT(".workspace");
		FPaths::NormalizeFilename(CachedSolutionPath);
	}

	return CachedSolutionPath;
}

bool FCodeLiteSourceCodeAccessor::Launch(const TArray<FString>& InArgs)
{
	if (Location.IsValid())
	{
		FString ArgsString;
		for (const FString& Arg : InArgs)
		{
			ArgsString.Append(Arg);
			ArgsString.Append(TEXT(" "));
		}

		uint32 ProcessID;
		FProcHandle hProcess = FPlatformProcess::CreateProc(*Location.URL, *ArgsString, true, false, false, &ProcessID, 0, nullptr, nullptr, nullptr);
		if (hProcess.IsValid())
		{
			FPlatformProcess::CloseProc(hProcess);
			return true;
		}
		else
		{
			return false;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
