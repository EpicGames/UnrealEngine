// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceCodeAccessor.h"

class FCodeLiteSourceCodeAccessor : public ISourceCodeAccessor
{
public:

	/**
	 * Call if we've likely modified the availability of the source code accessor. 
	 */
	virtual void RefreshAvailability() override;

	/**
	 * Can we access source code.
	 */
	virtual bool CanAccessSourceCode() const override;

	/**
	 * Get the name of this accessor. Will be used as a unique identifier.
	 */
	virtual FName GetFName() const override;

	/**
	 * Get the name of this accessor.
	 */
	virtual FText GetNameText() const override;

	/**
	 * Get the descriptor of this accessor.
	 */
	virtual FText GetDescriptionText() const override;

	/**
	 * Open the CodeLite Workspace for editing.
	 */
	virtual bool OpenSolution() override;

	/**
	 * Open the CodeLite Workspace for editing.
	 */
	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) override;

	/**
	 * Open the CodeLite Workspace for editing.
	 */
	virtual bool DoesSolutionExist() const override;

	/**
	 * Open a file at a specific line and optional column.
	 */
	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) override;

	/**
	 * Open a group of files.
	 */
	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) override;

	/**
	 * Add a group of files.
	 */
	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) override;

	/**
	 * Save all open files.
	 */
	virtual bool SaveAllOpenDocuments() const override;

	/**
	 * Tick this source code accessor
	 * @param DeltaTime Delta time (in seconds) since the last call to Tick
	 */
	virtual void Tick(const float DeltaTime) override;

	/**
	 * Initialize the accessor.
	 */
	void Startup();

	/**
	 * Deinitialize the accessor.
	 */
	void Shutdown();

private:

	struct FLocation
	{
		bool IsValid() const
		{
			return URL.Len() > 0;
		}

		FString URL;
	};

	/** Location instance */
	FLocation Location;

	/** String storing the solution path obtained from the module manager to avoid having to use it on a thread */
	mutable FString CachedSolutionPath;

	/** Critical section for updating SolutionPath */
	mutable FCriticalSection CachedSolutionPathCriticalSection;

	/** Gets solution path */
	FString GetSolutionPath() const;

	bool Launch(const TArray<FString>& InArgs);
};
